// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NonRealtimeAudioRenderer : ModuleRules
	{
		public NonRealtimeAudioRenderer(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateIncludePathModuleNames.Add("TargetPlatform");

            PublicIncludePathModuleNames.Add("AudioMixer");

            PrivateDependencyModuleNames.AddRange(
            new string[] {
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "AudioMixerCore",
                    "SignalProcessing",
                    "AudioMixer",
                }
			);

            AddEngineThirdPartyPrivateStaticDependencies(Target,
					"UEOgg",
					"Vorbis",
					"VorbisFile",
					"libOpus",
					"UELibSampleRate"
					);
        }
	}
}
