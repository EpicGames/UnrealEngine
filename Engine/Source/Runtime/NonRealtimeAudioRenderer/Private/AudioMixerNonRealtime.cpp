// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixer.h"
#include "AudioMixerDevice.h"
#include "AudioMixerPlatformNonRealtime.h"
#include "Modules/ModuleManager.h"

class FAudioMixerModuleNonRealtime : public IAudioDeviceModule
{
public:
	virtual bool IsAudioMixerModule() const override { return true; }

	virtual FAudioDevice* CreateAudioDevice() override
	{
		return new Audio::FMixerDevice(new Audio::FMixerPlatformNonRealtime());
	}
};

IMPLEMENT_MODULE(FAudioMixerModuleNonRealtime, NonRealtimeAudioRenderer);
