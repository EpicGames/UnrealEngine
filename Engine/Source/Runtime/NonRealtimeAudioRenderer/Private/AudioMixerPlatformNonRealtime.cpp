// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerPlatformNonRealtime.h"
#include "AudioMixer.h"
#include "AudioMixerDevice.h"
#include "AudioPluginUtilities.h"
#include "HAL/PlatformAffinity.h"
#include "Misc/App.h"

#ifndef HAS_COMPRESSED_AUDIO_INFO_CLASS
#define HAS_COMPRESSED_AUDIO_INFO_CLASS 0
#endif

#include "Interfaces/IAudioFormat.h"
#include "AudioDevice.h"

#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "AudioCompressionSettingsUtils.h"

static int32 DefaultRenderFrameSizeCvar = 256;
FAutoConsoleVariableRef CVarDefaultRenderFrameSize(
	TEXT("au.nrt.RenderFrameSize"),
	DefaultRenderFrameSizeCvar,
	TEXT("Selects the number of frames to render in a single callback .\n")
	TEXT("n: Number of frames to render."),
	ECVF_Default);

static int32 RenderEveryTickCvar = 1;
FAutoConsoleVariableRef CVarRenderEveryTick(
	TEXT("au.nrt.RenderEveryTick"),
	RenderEveryTickCvar,
	TEXT("When set to 1, calls the RenderAudio call every tick.\n")
	TEXT("n: Number of frames to render."),
	ECVF_Default);

namespace Audio
{
	FMixerPlatformNonRealtime::FMixerPlatformNonRealtime(float InSampleRate /*= 48000*/, float InNumChannels /*= 2*/)
		: SampleRate(InSampleRate)
		, NumChannels(InNumChannels)
		, TotalDurationRendered(0.0)
		, TotalDesiredRender(0.0)
		, TickDelta(0.0)
		, bIsInitialized(false)
		, bIsDeviceOpen(false)
	{
	}

	FMixerPlatformNonRealtime::~FMixerPlatformNonRealtime()
	{
	}

	void FMixerPlatformNonRealtime::RenderAudio(double NumSecondsToRender)
	{
		if (!bIsInitialized || !bIsDeviceOpen)
		{
			return;
		}

		const double TimePerCallback = ((double) AudioStreamInfo.NumOutputFrames) / AudioStreamInfo.DeviceInfo.SampleRate;

		// Increment how much audio time the user wants to have been rendered. 
		TotalDesiredRender += NumSecondsToRender;

		// Keep rendering audio until we surpass their desired time, TimePerCallback may be much smaller than NumSecondsToRender.
		while (TotalDurationRendered < TotalDesiredRender)
		{
			OutputBuffer.MixNextBuffer();

			ReadNextBuffer();
			TotalDurationRendered += TimePerCallback;
		}
	}

	void FMixerPlatformNonRealtime::OpenFileToWriteAudioTo(const FString& OutPath)
	{
		// Construct full path:
		FString AbsoluteFilePath;

		const bool bIsRelativePath = FPaths::IsRelative(OutPath);
		if (bIsRelativePath)
		{
			AbsoluteFilePath = FPaths::ProjectSavedDir() + OutPath;
			AbsoluteFilePath = FPaths::ConvertRelativePathToFull(AbsoluteFilePath);
		}
		else
		{
			AbsoluteFilePath = OutPath;
		}

		FSoundQualityInfo QualityInfo;
		QualityInfo.SampleRate = SampleRate;
		QualityInfo.NumChannels = NumChannels;
		QualityInfo.Quality = 100;

		// Gotcha for bouncing wav files: this has to be filled in.
		QualityInfo.Duration = 5.0f;
		QualityInfo.SampleDataSize = 5.0f * SampleRate * NumChannels * sizeof(int16);

		AudioFileWriter.Reset(new FAudioFileWriter(AbsoluteFilePath, QualityInfo));
	}

	void FMixerPlatformNonRealtime::CloseFile()
	{
		AudioFileWriter.Reset();
	}

	bool FMixerPlatformNonRealtime::InitializeHardware()
	{
		if (bIsInitialized)
		{
			return false;
		}

		bIsInitialized = true;

		TickDelta = FApp::GetDeltaTime();

		return true;
	}

	bool FMixerPlatformNonRealtime::TeardownHardware()
	{
		if (!bIsInitialized)
		{
			return false;
		}

		bIsInitialized = false;

		return true;
	}

	bool FMixerPlatformNonRealtime::IsInitialized() const
	{
		return bIsInitialized;
	}

	bool FMixerPlatformNonRealtime::GetNumOutputDevices(uint32& OutNumOutputDevices)
	{
		if (!bIsInitialized)
		{
			return false;
		}
		
		OutNumOutputDevices = 1;

		return true;
	}

	bool FMixerPlatformNonRealtime::GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo)
	{
		if (!bIsInitialized)
		{
			return false;
		}

		OutInfo.bIsSystemDefault = true;
		OutInfo.SampleRate = SampleRate;
		OutInfo.DeviceId = 0;
		OutInfo.Format = EAudioMixerStreamDataFormat::Float;
		OutInfo.Name = TEXT("Non-realtime Renderer");
		OutInfo.NumChannels = NumChannels;

		OutInfo.OutputChannelArray.Add(EAudioMixerChannel::FrontLeft);
		OutInfo.OutputChannelArray.Add(EAudioMixerChannel::FrontRight);
		OutInfo.OutputChannelArray.Add(EAudioMixerChannel::FrontCenter);
		OutInfo.OutputChannelArray.Add(EAudioMixerChannel::LowFrequency);
		OutInfo.OutputChannelArray.Add(EAudioMixerChannel::BackLeft);
		OutInfo.OutputChannelArray.Add(EAudioMixerChannel::BackRight);
		OutInfo.OutputChannelArray.Add(EAudioMixerChannel::SideLeft);
		OutInfo.OutputChannelArray.Add(EAudioMixerChannel::SideRight);

		return true;
	}

	bool FMixerPlatformNonRealtime::GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const
	{
		OutDefaultDeviceIndex = 0;
		return true;
	}

	bool FMixerPlatformNonRealtime::OpenAudioStream(const FAudioMixerOpenStreamParams& Params)
	{
		if (!bIsInitialized)
		{
			return false;
		}

		if (bIsDeviceOpen)
		{
			return false;
		}

		OpenStreamParams = Params;
		//OpenStreamParams.NumFrames = DefaultRenderFrameSizeCvar;
		

		AudioStreamInfo.Reset();

		AudioStreamInfo.OutputDeviceIndex = OpenStreamParams.OutputDeviceIndex;
		AudioStreamInfo.NumOutputFrames = OpenStreamParams.NumFrames;
		AudioStreamInfo.NumBuffers = OpenStreamParams.NumBuffers;
		AudioStreamInfo.AudioMixer = OpenStreamParams.AudioMixer;


		if (!GetOutputDeviceInfo(AudioStreamInfo.OutputDeviceIndex, AudioStreamInfo.DeviceInfo))
		{
			return false;
		}

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Open;
		bIsDeviceOpen = true;

		return true;
	}

	FAudioPlatformDeviceInfo FMixerPlatformNonRealtime::GetPlatformDeviceInfo() const
	{
		return AudioStreamInfo.DeviceInfo;
	}

	bool FMixerPlatformNonRealtime::CloseAudioStream()
	{
		if (!bIsInitialized || AudioStreamInfo.StreamState == EAudioOutputStreamState::Closed)
		{
			return false;
		}

		if (bIsDeviceOpen && !StopAudioStream())
		{
			return false;
		}

		bIsDeviceOpen = false;

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Closed;

		return true;
	}

	bool FMixerPlatformNonRealtime::StartAudioStream()
	{
		// Start generating audio with our output source voice
		BeginGeneratingAudio();

		return true;
	}

	bool FMixerPlatformNonRealtime::StopAudioStream()
	{
		return true;
	}

	bool FMixerPlatformNonRealtime::CheckAudioDeviceChange()
	{
		return false;
	}

	bool FMixerPlatformNonRealtime::MoveAudioStreamToNewAudioDevice(const FString& InNewDeviceId)
	{
		return true;
	}

	void FMixerPlatformNonRealtime::ResumePlaybackOnNewDevice()
	{
		int32 PoppedSampleCount;
		TArrayView<const uint8> PoppedAudio = OutputBuffer.PopBufferData(PoppedSampleCount);
		SubmitBuffer(PoppedAudio.GetData());
		check(OpenStreamParams.NumFrames * AudioStreamInfo.DeviceInfo.NumChannels == OutputBuffer.GetNumSamples());

		AudioRenderEvent->Trigger();
	}

	void FMixerPlatformNonRealtime::SubmitBuffer(const uint8* Buffer)
	{
		// Do actual buffer submissions here.
		if (AudioFileWriter.IsValid())
		{
			AudioFileWriter->PushAudio((const float*) Buffer, NumChannels * AudioStreamInfo.NumOutputFrames);
		}
	}

	FString FMixerPlatformNonRealtime::GetDefaultDeviceName()
	{
		//GConfig->GetString(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("AudioDevice"), WindowsAudioDeviceName, GEngineIni);
		return FString();
	}

	FAudioPlatformSettings FMixerPlatformNonRealtime::GetPlatformSettings() const
	{
		return FAudioPlatformSettings::GetPlatformSettings(FPlatformProperties::GetRuntimeSettingsClassName());
	}

	void FMixerPlatformNonRealtime::OnHardwareUpdate()
	{
		if (RenderEveryTickCvar)
		{
			RenderAudio(TickDelta);
		}
	}

	bool FMixerPlatformNonRealtime::IsNonRealtime() const
	{
		return true;
	}

	void FMixerPlatformNonRealtime::FadeOut()
	{
		bFadedOut = true;
		FadeVolume = 0.f;
	}

	uint32 FMixerPlatformNonRealtime::RunInternal()
	{
		// Not used.
		return 0;
	}

	void FMixerPlatformNonRealtime::FadeIn()
	{
		bFadedOut = false;
		FadeVolume = 1.0f;
	}
}
