// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "DSP/FileEncoder.h"

// Any platform defines
namespace Audio
{
	class FMixerPlatformNonRealtime : public IAudioMixerPlatformInterface
	{

	public:

		FMixerPlatformNonRealtime(float InSampleRate = 48000, float InNumChannels = 2);
		~FMixerPlatformNonRealtime();

		NONREALTIMEAUDIORENDERER_API void RenderAudio(double NumSecondsToRender);
		void OpenFileToWriteAudioTo(const FString& OutPath);
		void CloseFile();

		//~ Begin IAudioMixerPlatformInterface
		virtual FString GetPlatformApi() const override { return TEXT("NonRealtime"); }
		virtual bool InitializeHardware() override;
		virtual bool CheckAudioDeviceChange() override;
		virtual bool TeardownHardware() override;
		virtual bool IsInitialized() const override;
		virtual bool GetNumOutputDevices(uint32& OutNumOutputDevices) override;
		virtual bool GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo) override;
		virtual bool GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const override;
		virtual bool OpenAudioStream(const FAudioMixerOpenStreamParams& Params) override;
		virtual bool CloseAudioStream() override;
		virtual bool StartAudioStream() override;
		virtual bool StopAudioStream() override;
		virtual bool MoveAudioStreamToNewAudioDevice(const FString& InNewDeviceId) override;
		virtual void ResumePlaybackOnNewDevice() override;
		virtual FAudioPlatformDeviceInfo GetPlatformDeviceInfo() const override;
		virtual void SubmitBuffer(const uint8* Buffer) override;
		virtual FString GetDefaultDeviceName() override;
		virtual FAudioPlatformSettings GetPlatformSettings() const override;
		virtual void OnHardwareUpdate() override;
		virtual bool IsNonRealtime() const override;
		virtual void FadeOut() override;

		virtual void FadeIn() override;

		//~ End IAudioMixerPlatformInterface

	private:
		float SampleRate;
		int32 NumChannels;

		// How much audio time has actually been rendered? Incremented by RenderAudio at fixed precision.
		double TotalDurationRendered;
		// How much time does the user want to have rendered? This prevents drift over time where we render more audio on a given frame than asked.
		double TotalDesiredRender;

		// This is retrieved from the tick interval on InitializeHardware.
		double TickDelta;

		uint32 bIsInitialized : 1;
		uint32 bIsDeviceOpen : 1;

		TUniquePtr<FAudioFileWriter> AudioFileWriter;

	protected:
		virtual uint32 RunInternal() override;

	};

}

